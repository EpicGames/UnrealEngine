{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NiagaraPreviewContent",
	"Description": "Contains movie files used in Niagara",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault" : true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraPreviewContent",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}